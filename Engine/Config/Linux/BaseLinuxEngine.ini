[OnlineSubsystem]
LocalPlatformName=LNX

[PlatformPaths]
UnrealBuildTool=Engine/Build/BatchFiles/Linux/Build.sh

[CrashReportClient]
bAgreeToCrashUpload=false

[ShaderPlatformConfig VULKAN_SM6]
; Bindless is necessary for Vulkan ray tracing
BindlessConfiguration=All

[Linux.SDL]
; Override the SDL video driver. Common values: "wayland", "x11", "dummy"
; If not set, SDL auto-detects based on your system.
; The SDL_VIDEO_DRIVER environment variable takes precedence over this setting.
; The -sdlvideodriver= command line argument also takes precedence over this setting.
;VideoDriver=

[/Script/Engine.RendererSettings]
; Substrate settings for more advanced shading on PC and editor by default.
r.Substrate.RoughDiffuse=1
r.Substrate.ShadingQuality=1
r.Substrate.TileCoord8bits=0
r.Substrate.Glints=1
r.Substrate.ClosuresPerPixel=4
r.Refraction.Blur=1
