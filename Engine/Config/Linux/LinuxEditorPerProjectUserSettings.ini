
; -------------------------------------------------------------------------------------------------
; IMPORTANT:  EditorPerProjectUserSettings.ini has special behavior!
;
; This .ini file contains the defaults for many editor and engine "preferences".  These preferences
; are saved to an EditorPerProjectUserSettings.ini file in the user's /<Game>/Saved/Config/ directory. 
;
; If you change the default for a preference in here, that change will *NOT* automatically 
; propagate to users who already have saved preferences.  The idea is that we must preserve
; their existing settings and never want to clobber entries in that file.
;
; If you add a new preference, the new key and value *WILL* be propagated to the user's
; EditorPerProjectUserSettings.ini file.  After we load the user's existing settings, we'll merge in
; any missing keys and values that are present in these defaults.
;
; One easy technique for forcing a "reset" of an already-existing preference is to simply
; rename the variable for that preference.  The config system will treat it as a newly-added
; key and will propagate the change to existing users' preferences.
; -------------------------------------------------------------------------------------------------

[/Script/GraphEditor.GraphEditorSettings]
bUseHighPrecisionMouseMovement=False
bUseInterpolationWithManualPanning=True
