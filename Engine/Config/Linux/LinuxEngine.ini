[Audio]
; Defines a platform-specific volume headroom (in dB) for audio to provide better platform consistency with respect to volume levels.
PlatformHeadroomDB=0

[SystemSettings]
r.setres=1280x720
fx.NiagaraAllowRuntimeScalabilityChanges=1
;Perplatform to PerQualityLevel conversion mapping for platform groups
QualityLevelMapping="high"
r.Shadow.DetectVertexShaderLayerAtRuntime=1

[TextureStreaming]
; PoolSizeVRAMPercentage is how much percentage of GPU Dedicated VRAM should be used as a TexturePool cache for streaming textures (0 - unlimited streaming)
PoolSizeVRAMPercentage=70

[DeviceProfileManager]
DeviceProfileSelectionModule="LinuxDeviceProfileSelector"

[ConsoleVariables]
; larger timeout since drivers may take longer time
g.TimeoutForBlockOnRenderFence=60000
memory.MemoryPressureCriticalThresholdPercent=10

[PlatformCrypto]
PlatformRequiresDataCrypto=True

[/Script/Engine.RendererSettings]
r.Shaders.RemoveUnusedInterpolators=1
