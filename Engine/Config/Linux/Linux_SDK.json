{
	"MainVersion" : "v26_clang-20.1.8-rockylinux8",
	"MinVersion" : "v26_clang-20.1.8-rockylinux8",
	"MaxVersion" : "v26_clang-20.1.8-rockylinux8",

	"AutoSDKPlatform" : "Linux_x64"
}
